{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "BaseCharacterFXEditor",
	"Description": "Base classes for character FX asset editors",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "BaseCharacterFXEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true
		}
	]
}