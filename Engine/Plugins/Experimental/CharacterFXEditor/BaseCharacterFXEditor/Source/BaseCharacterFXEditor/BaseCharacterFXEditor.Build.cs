// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BaseCharacterFXEditor : ModuleRules
{
	public BaseCharacterFXEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "BCFXEd";

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"UnrealEd",
				"Core",
				"InputCore",
				"InteractiveToolsFramework",
				"EditorStyle",
				"Slate",
				"SlateCore",
				"CoreUObject",
				"Engine",
				"Projects",
				"AdvancedPreviewScene",
				"ContentBrowser",
				"DynamicMesh",
				"EditorFramework",
				"EditorInteractiveToolsFramework",
				"ModelingComponents",
				"EditorSubsystem",
				"GeometryCore",
				"GeometryFramework",
				"LevelEditor",
				"StatusBar",
				"ToolMenus",
				"ToolWidgets",
				"WorkspaceMenuStructure",
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
