// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseCharacterFXEditor.h"
#include "BaseCharacterFXEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseCharacterFXEditor)

void UBaseCharacterFXEditor::Initialize(const TArray<TObjectPtr<UObject>>& InObjects, const TSubclassOf<AActor>& InPreviewClass)
{
	OriginalObjectsToEdit = InObjects;

	// This will do a variety of things including registration of the asset editor, creation of the toolkit
	// (via CreateToolkit()), and creation of the editor mode manager within the toolkit.
	// The asset editor toolkit will do the rest of the necessary initialization inside its PostInitAssetEditor.
	UAssetEditor::Initialize();
}

IAssetEditorInstance* UBaseCharacterFXEditor::GetInstanceInterface()
{ 
	return ToolkitInstance; 
}

void UBaseCharacterFXEditor::GetObjectsToEdit(TArray<UObject*>& OutObjects)
{
	OutObjects.Append(OriginalObjectsToEdit);
	check(OutObjects.Num() > 0);
}
