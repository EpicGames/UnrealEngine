// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseCharacterFXEditorMode.h"

#include "Components/DynamicMeshComponent.h"
#include "AssetEditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ModelingToolTargetUtil.h"
#include "BaseCharacterFXEditorCommands.h"
#include "BaseCharacterFXEditorModeToolkit.h"
#include "ToolTargets/ToolTarget.h"
#include "Engine/Selection.h"
#include "InteractiveToolQueryInterfaces.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseCharacterFXEditorMode)

#define LOCTEXT_NAMESPACE "UBaseCharacterFXEditorMode"

void UBaseCharacterFXEditorMode::Enter()
{
	Super::Enter();

	AddToolTargetFactories();
	RegisterTools();
	ActivateDefaultTool();
}

void UBaseCharacterFXEditorMode::Exit()
{
	// ToolsContext->EndTool only shuts the tool on the next tick, and ToolsContext->DeactivateActiveTool is
	// inaccessible, so we end up having to do this to force the shutdown right now.
	GetToolManager()->DeactivateTool(EToolSide::Mouse, EToolShutdownType::Cancel);

	OriginalObjectsToEdit.Reset();
	ToolTargets.Reset();

	Super::Exit();
}


void UBaseCharacterFXEditorMode::AcceptActiveToolActionOrTool()
{
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
		if (CancelAPI && CancelAPI->SupportsNestedAcceptCommand() && CancelAPI->CanCurrentlyNestedAccept())
		{
			bool bAccepted = CancelAPI->ExecuteNestedAcceptCommand();
			if (bAccepted)
			{
				return;
			}
		}
	}

	const EToolShutdownType ShutdownType = GetInteractiveToolsContext()->CanAcceptActiveTool() ? EToolShutdownType::Accept : EToolShutdownType::Completed;
	GetInteractiveToolsContext()->EndTool(ShutdownType);
}


void UBaseCharacterFXEditorMode::CancelActiveToolActionOrTool()
{
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
		if (CancelAPI && CancelAPI->SupportsNestedCancelCommand() && CancelAPI->CanCurrentlyNestedCancel())
		{
			bool bCancelled = CancelAPI->ExecuteNestedCancelCommand();
			if (bCancelled)
			{
				return;
			}
		}
	}

	const EToolShutdownType ShutdownType = GetInteractiveToolsContext()->CanCancelActiveTool() ? EToolShutdownType::Cancel : EToolShutdownType::Completed;
	GetInteractiveToolsContext()->EndTool(ShutdownType);
}

void UBaseCharacterFXEditorMode::InitializeTargets(const TArray<TObjectPtr<UObject>>& AssetsIn)
{
	OriginalObjectsToEdit = AssetsIn;

	CreateToolTargets(AssetsIn);
}

#undef LOCTEXT_NAMESPACE
