// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseCharacterFXEditorModeToolkit.h"
#include "BaseCharacterFXEditorCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "SPrimaryButton.h"
#include "Tools/UEdMode.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "EditorModeManager.h"
#include "AssetEditorModeManager.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FBaseCharacterFXEditorModeToolkit"

FBaseCharacterFXEditorModeToolkit::FBaseCharacterFXEditorModeToolkit()
{
	// Construct the panel that we will populate in GetInlineContent().
	// This could probably be done in Init() instead, but constructor
	// makes it easy to guarantee that GetInlineContent() will always
	// be ready to work.

	SAssignNew(ToolkitWidget, SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	[
		SAssignNew(ToolWarningArea, STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f))) //TODO: This probably needs to not be hardcoded
		.Text(FText::GetEmpty())
		.Visibility(EVisibility::Collapsed)
	]

	+ SVerticalBox::Slot()
	[
		SAssignNew(ToolDetailsContainer, SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
	];
}

FBaseCharacterFXEditorModeToolkit::~FBaseCharacterFXEditorModeToolkit()
{
	UEdMode* Mode = GetScriptableEditorMode().Get();
	if (ensure(Mode))
	{
		UEditorInteractiveToolsContext* Context = Mode->GetInteractiveToolsContext();
		if (ensure(Context))
		{
			Context->OnToolNotificationMessage.RemoveAll(this);
			Context->OnToolWarningMessage.RemoveAll(this);
		}
	}
}

void FBaseCharacterFXEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FModeToolkit::Init(InitToolkitHost, InOwningMode);	

	// Set up tool message areas
	ClearNotification();
	ClearWarning();
	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolNotificationMessage.AddSP(this, &FBaseCharacterFXEditorModeToolkit::PostNotification);
	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolWarningMessage.AddSP(this, &FBaseCharacterFXEditorModeToolkit::PostWarning);

	// Hook up the tool detail panel
	ToolDetailsContainer->SetContent(DetailsView.ToSharedRef());
	
	// Set up the overlay. Largely copied from ModelingToolsEditorModeToolkit.
	// TODO: We should put some of the shared code in some common place (jira UE-158732)
	SAssignNew(ViewportOverlayWidget, SHorizontalBox)

	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
		.Padding(8.f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this] () { return ActiveToolIcon; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text(this, &FBaseCharacterFXEditorModeToolkit::GetActiveToolDisplayName)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("OverlayAccept", "Accept"))
				.ToolTipText(LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]"))
				.OnClicked_Lambda([this]() { 
					GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept);
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanAcceptActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Button")
				.TextStyle( FAppStyle::Get(), "DialogButtonText" )
				.Text(LOCTEXT("OverlayCancel", "Cancel"))
				.ToolTipText(LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { 
					GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel); 
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCancelActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("OverlayComplete", "Complete"))
				.ToolTipText(LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { 
					GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed); 
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() {
					return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCompleteActiveTool();
				})
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCompleteActiveTool() ? EVisibility::Visible : EVisibility::Collapsed; })
			]
		]	
	];
}

void FBaseCharacterFXEditorModeToolkit::UpdateActiveToolProperties()
{
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager()->GetActiveTool(EToolSide::Left);
	if (CurTool != nullptr)
	{
		DetailsView->SetObjects(CurTool->GetToolProperties(true));
	}
}

void FBaseCharacterFXEditorModeToolkit::InvalidateCachedDetailPanelState(UObject* ChangedObject)
{
	DetailsView->InvalidateCachedState();
}

void FBaseCharacterFXEditorModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FModeToolkit::OnToolStarted(Manager, Tool);
	
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager()->GetActiveTool(EToolSide::Left);
	CurTool->OnPropertySetsModified.AddSP(this, &FBaseCharacterFXEditorModeToolkit::UpdateActiveToolProperties);
	CurTool->OnPropertyModifiedDirectlyByTool.AddSP(this, &FBaseCharacterFXEditorModeToolkit::InvalidateCachedDetailPanelState);

	ActiveToolName = Tool->GetToolInfo().ToolDisplayName;

	FString ActiveToolIdentifier = GetScriptableEditorMode()->GetToolManager()->GetActiveToolName(EToolSide::Mouse);
	ActiveToolIdentifier.InsertAt(0, ".");

	ActiveToolIcon = GetActiveToolIcon(ActiveToolIdentifier);

	GetToolkitHost()->AddViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
}

void FBaseCharacterFXEditorModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FModeToolkit::OnToolEnded(Manager, Tool);

	ActiveToolName = FText::GetEmpty();
	ClearNotification();
	ClearWarning();

	if (IsHosted())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}

	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager()->GetActiveTool(EToolSide::Left);
	if (CurTool)
	{
		CurTool->OnPropertySetsModified.RemoveAll(this);
		CurTool->OnPropertyModifiedDirectlyByTool.RemoveAll(this);
	}
}

// Place tool category names here, for creating the tool palette below
const FName FBaseCharacterFXEditorModeToolkit::ToolsTabName = TEXT("Tools");
const TArray<FName> FBaseCharacterFXEditorModeToolkit::PaletteNames_Standard = { ToolsTabName };

void FBaseCharacterFXEditorModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames = PaletteNames_Standard;
}

FText FBaseCharacterFXEditorModeToolkit::GetToolPaletteDisplayName(FName Palette) const
{
	return FText::FromName(Palette);
}


void FBaseCharacterFXEditorModeToolkit::PostNotification(const FText& Message)
{
	ClearNotification();

	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		ActiveToolMessageHandle = GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PushStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), Message);
	}
}

void FBaseCharacterFXEditorModeToolkit::ClearNotification()
{
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessageHandle);
	}
	ActiveToolMessageHandle.Reset();
}

void FBaseCharacterFXEditorModeToolkit::PostWarning(const FText& Message)
{
	if (Message.IsEmpty())
	{
		ClearWarning();
	}
	else
	{
		ToolWarningArea->SetText(Message);
		ToolWarningArea->SetVisibility(EVisibility::Visible);
	}
}

void FBaseCharacterFXEditorModeToolkit::ClearWarning()
{
	ToolWarningArea->SetText(FText());
	ToolWarningArea->SetVisibility(EVisibility::Collapsed);
}


#undef LOCTEXT_NAMESPACE
