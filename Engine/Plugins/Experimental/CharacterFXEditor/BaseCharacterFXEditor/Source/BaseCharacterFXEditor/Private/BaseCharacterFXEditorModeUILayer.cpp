// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseCharacterFXEditorModeUILayer.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Toolkits/IToolkit.h"
#include "BaseCharacterFXEditorToolkit.h"
#include "BaseCharacterFXEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseCharacterFXEditorModeUILayer)

FBaseCharacterFXEditorModeUILayer::FBaseCharacterFXEditorModeUILayer(const IToolkitHost* InToolkitHost) :
	FAssetEditorModeUILayer(InToolkitHost)
{}

void FBaseCharacterFXEditorModeUILayer::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	if (!Toolkit->IsAssetEditor())		// We only want to host Mode toolkits
	{
		FAssetEditorModeUILayer::OnToolkitHostingStarted(Toolkit);
		HostedToolkit = Toolkit;
		Toolkit->SetModeUILayer(SharedThis(this));
		Toolkit->RegisterTabSpawners(ToolkitHost->GetTabManager().ToSharedRef());
		RegisterModeTabSpawners();
		OnToolkitHostReadyForUI.ExecuteIfBound();
	}
}

void FBaseCharacterFXEditorModeUILayer::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	if (HostedToolkit.IsValid() && HostedToolkit.Pin() == Toolkit)	// don't execute OnToolkitHostShutdownUI if the input Toolkit isn't the one we're hosting
	{
		FAssetEditorModeUILayer::OnToolkitHostingFinished(Toolkit);
	}
}

TSharedPtr<FWorkspaceItem> FBaseCharacterFXEditorModeUILayer::GetModeMenuCategory() const
{
	check(CharacterFXEditorMenuCategory);
	return CharacterFXEditorMenuCategory;
}

void FBaseCharacterFXEditorModeUILayer::SetModeMenuCategory(TSharedPtr<FWorkspaceItem> MenuCategoryIn)
{
	CharacterFXEditorMenuCategory = MenuCategoryIn;
}


const FName UBaseCharacterFXEditorUISubsystem::EditorSidePanelAreaName = "BaseCharacterFXEditorSidePanelArea";

void UBaseCharacterFXEditorUISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FName ConcreteModuleName = GetModuleName();
	if (ConcreteModuleName != FName(""))
	{
		FBaseCharacterFXEditorModule& ConcreteCharacterFXEditorModule = FModuleManager::GetModuleChecked<FBaseCharacterFXEditorModule>(ConcreteModuleName);
		ConcreteCharacterFXEditorModule.OnRegisterLayoutExtensions().AddUObject(this, &UBaseCharacterFXEditorUISubsystem::RegisterLayoutExtensions);
	}
}

void UBaseCharacterFXEditorUISubsystem::Deinitialize()
{
	FName ConcreteModuleName = GetModuleName();
	if (ConcreteModuleName != FName(""))
	{
		FBaseCharacterFXEditorModule& ConcreteCharacterFXEditorModule = FModuleManager::GetModuleChecked<FBaseCharacterFXEditorModule>(ConcreteModuleName);
		ConcreteCharacterFXEditorModule.OnRegisterLayoutExtensions().RemoveAll(this);
	}
}

void UBaseCharacterFXEditorUISubsystem::RegisterLayoutExtensions(FLayoutExtender& Extender)
{
	FTabManager::FTab NewTab(FTabId(UAssetEditorUISubsystem::TopLeftTabID), ETabState::ClosedTab);
	Extender.ExtendStack(UBaseCharacterFXEditorUISubsystem::EditorSidePanelAreaName, ELayoutExtensionPosition::After, NewTab);
}


