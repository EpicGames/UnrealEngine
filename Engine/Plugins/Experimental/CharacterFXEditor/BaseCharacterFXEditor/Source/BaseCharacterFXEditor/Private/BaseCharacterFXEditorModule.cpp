// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseCharacterFXEditorModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FBaseCharacterFXEditorModule"

void FBaseCharacterFXEditorModule::StartupModule()
{
}

void FBaseCharacterFXEditorModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FBaseCharacterFXEditorModule, BaseCharacterFXEditor)
