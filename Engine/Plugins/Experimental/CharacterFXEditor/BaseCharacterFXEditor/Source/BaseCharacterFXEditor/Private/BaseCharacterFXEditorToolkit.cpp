// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseCharacterFXEditorToolkit.h"
#include "BaseCharacterFXEditorModule.h"
#include "BaseCharacterFXEditorMode.h"
#include "BaseCharacterFXEditorModeUILayer.h"
#include "SBaseCharacterFXEditorViewport.h"
#include "AdvancedPreviewScene.h"
#include "Framework/Docking/LayoutExtender.h"
#include "AssetEditorModeManager.h"
#include "AdvancedPreviewScene.h"
#include "SAssetEditorViewport.h"
#include "EditorViewportTabContent.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Tools/UAssetEditor.h"

#define LOCTEXT_NAMESPACE "BaseCharacterFXEditorToolkit"

FBaseCharacterFXEditorToolkit::FBaseCharacterFXEditorToolkit(UAssetEditor* InOwningAssetEditor, const FName& ModuleName)
	: FBaseAssetToolkit(InOwningAssetEditor) 
{
	// We will replace the StandaloneDefaultLayout that our parent class gave us with 
	// a custom layout. We define explicit ExtensionIds on the stacks to reference them 
	// later when the UILayer provides layout extensions. 
	//
	// Note: Changes to the layout should include a increment to the layout name suffix, i.e.
	// CharacterFXEditorLayout[X] -> CharacterFXEditorLayout[X+1]. Otherwise, layouts may be messed up
	// without a full reset to layout defaults inside the editor.

	StandaloneDefaultLayout = FTabManager::NewLayout(FName("CharacterFXEditorLayout1"))
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.2f)
				->SetExtensionId(UBaseCharacterFXEditorUISubsystem::EditorSidePanelAreaName)
				->SetHideTabWell(true)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.4f)
				->AddTab(ViewportTabID, ETabState::OpenedTab)
				->SetExtensionId("ViewportArea")
				->SetHideTabWell(true)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.3f)
				->AddTab(DetailsTabID, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
		)
	);

	// Add any extenders specified by the UISubsystem
	// The extenders provide defined locations for FModeToolkit to attach
	// tool palette tabs and detail panel tabs
	LayoutExtender = MakeShared<FLayoutExtender>();
	FBaseCharacterFXEditorModule* Module = &FModuleManager::LoadModuleChecked<FBaseCharacterFXEditorModule>(ModuleName);
	Module->OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
	StandaloneDefaultLayout->ProcessExtensions(*LayoutExtender);

	// We could create the preview scene in CreateEditorViewportClient() the way that FBaseAssetToolkit
	// does, but it seems more intuitive to create them right off the bat and pass it in later. 
	FPreviewScene::ConstructionValues SceneArgs;
	ObjectScene = MakeShared<FAdvancedPreviewScene>(SceneArgs);
}

void FBaseCharacterFXEditorToolkit::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingStarted(Toolkit);
}

void FBaseCharacterFXEditorToolkit::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingFinished(Toolkit);
}

bool FBaseCharacterFXEditorToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	// Note: This needs a bit of adjusting, because currently OnRequestClose seems to be 
	// called multiple times when the editor itself is being closed. We can take the route 
	// of NiagaraScriptToolkit and remember when changes are discarded, but this can cause
	// issues if the editor close sequence is interrupted due to some other asset editor.

	UBaseCharacterFXEditorMode* EdMode = Cast<UBaseCharacterFXEditorMode>(EditorModeManager->GetActiveScriptableMode(GetEditorModeId()));
	if (!EdMode)
	{
		// If we don't have a valid mode, because the OnRequestClose is currently being called multiple times,
		// simply return true because there's nothing left to do.
		return true;
	}

	// Give any active modes a chance to shutdown while the toolkit host is still alive
	// This is super important to do, otherwise currently opened tabs won't be marked as "closed".
	// This results in tabs not being properly recycled upon reopening the editor and tab
	// duplication for each opening event.
	GetEditorModeManager().DeactivateAllModes();

	return FAssetEditorToolkit::OnRequestClose(InCloseReason);
}

AssetEditorViewportFactoryFunction FBaseCharacterFXEditorToolkit::GetViewportDelegate()
{
	AssetEditorViewportFactoryFunction TempViewportDelegate = [this](FAssetEditorViewportConstructionArgs InArgs)
	{
		return SNew(SBaseCharacterFXEditorViewport, InArgs)
			.EditorViewportClient(ViewportClient);
	};
	return TempViewportDelegate;
}

void FBaseCharacterFXEditorToolkit::AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder)
{
	TSharedPtr<SBaseCharacterFXEditorViewport> ViewportWidget = StaticCastSharedPtr<SBaseCharacterFXEditorViewport>(ViewportTabContent->GetFirstViewport());
	ViewportWidget->AddOverlayWidget(InViewportOverlayWidget, ZOrder);
}

void FBaseCharacterFXEditorToolkit::RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget)
{
	TSharedPtr<SBaseCharacterFXEditorViewport> ViewportWidget = StaticCastSharedPtr<SBaseCharacterFXEditorViewport>(ViewportTabContent->GetFirstViewport());
	ViewportWidget->RemoveOverlayWidget(InViewportOverlayWidget);
}

void FBaseCharacterFXEditorToolkit::CreateEditorModeManager()
{
	EditorModeManager = MakeShared<FAssetEditorModeManager>();

	// The mode manager is the authority on what the world is for the mode and the tools context,
	// and setting the preview scene here makes our GetWorld() function return the preview scene
	// world instead of the normal level editor one. Important because that is where we create
	// any preview meshes, gizmo actors, etc.
	StaticCastSharedPtr<FAssetEditorModeManager>(EditorModeManager)->SetPreviewScene(ObjectScene.Get());
}

void FBaseCharacterFXEditorToolkit::CreateEditorModeUILayer()
{
	TSharedPtr<class IToolkitHost> PinnedToolkitHost = ToolkitHost.Pin();
	check(PinnedToolkitHost.IsValid());
	ModeUILayer = MakeShareable(new FBaseCharacterFXEditorModeUILayer(PinnedToolkitHost.Get()));
}

// Called from FBaseAssetToolkit::CreateWidgets to populate ViewportClient, but otherwise only used 
// in our own viewport delegate.
TSharedPtr<FEditorViewportClient> FBaseCharacterFXEditorToolkit::CreateEditorViewportClient() const
{
	// Note that we can't reliably adjust the viewport client here because we will be passing it
	// into the viewport created by the viewport delegate we get from GetViewportDelegate(), and
	// that delegate may (will) affect the settings based on FAssetEditorViewportConstructionArgs,
	// namely ViewportType.
	// Instead, we do viewport client adjustment in PostInitAssetEditor().
	check(EditorModeManager.IsValid());
	return MakeShared<FEditorViewportClient>(EditorModeManager.Get(), ObjectScene.Get());
}

void FBaseCharacterFXEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FBaseAssetToolkit::RegisterTabSpawners(InTabManager);
	EditorMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_CharacterFXEditor", "CharacterFX Editor"));
}

void FBaseCharacterFXEditorToolkit::PostInitAssetEditor()
{
	// We setup the ModeUILayer connection here, since the InitAssetEditor is closed off to us. 
	// Other editors perform this step elsewhere, but this is our best location.
	CreateEditorModeUILayer();
	ModeUILayer->SetModeMenuCategory(EditorMenuCategory);

	// Currently, aside from setting up all the UI elements, the toolkit also kicks off the
	// editor mode, which is the mode that the editor always works in (things are packaged into
	// a mode so that they can be moved to another asset editor if necessary).
	// We need the mode to be active to create the toolbox on the left.
	check(EditorModeManager.IsValid());
	EditorModeManager->ActivateMode(GetEditorModeId());
	UBaseCharacterFXEditorMode* EdMode = Cast<UBaseCharacterFXEditorMode>(EditorModeManager->GetActiveScriptableMode(GetEditorModeId()));
	check(EdMode);

	InitializeEdMode(EdMode);

	// Regardless of how the user has modified the layout, we're going to make sure that we have
	// a viewport that will allow our mode to receive ticks.
	// We don't need to invoke the tool palette tab anymore, since this is handled by
	// underlying infrastructure.
	if (!TabManager->FindExistingLiveTab(ViewportTabID) && bForceViewportTab)
	{
		TabManager->TryInvokeTab(ViewportTabID);
	}

	ViewportClient->FocusViewportOnBox(EdMode->SceneBoundingBox());

	if(ViewportClient->Viewport)
	{
		// We need the viewport client to start out focused, or else it won't get ticked until
		// we click inside it.
		ViewportClient->ReceivedFocus(ViewportClient->Viewport);
	}
}

void FBaseCharacterFXEditorToolkit::InitializeEdMode(UBaseCharacterFXEditorMode* EdMode)
{
	TArray<TObjectPtr<UObject>> ObjectsToEdit;
	OwningAssetEditor->GetObjectsToEdit(MutableView(ObjectsToEdit));

	EdMode->InitializeTargets(ObjectsToEdit);
}

#undef LOCTEXT_NAMESPACE
