// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBaseCharacterFXEditorViewport.h"
#include "SViewportToolBar.h"

#define LOCTEXT_NAMESPACE "SBaseCharacterFXEditorViewport"

void SBaseCharacterFXEditorViewport::AddOverlayWidget(TSharedRef<SWidget> OverlaidWidget, int32 ZOrder)
{
	ViewportOverlay->AddSlot(ZOrder)
	[
		OverlaidWidget
	];
}

void SBaseCharacterFXEditorViewport::RemoveOverlayWidget(TSharedRef<SWidget> OverlaidWidget)
{
	ViewportOverlay->RemoveSlot(OverlaidWidget);
}

#undef LOCTEXT_NAMESPACE
