// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"
#include "BaseCharacterFXEditor.generated.h"

#define UE_API BASECHARACTERFXEDITOR_API

/**
 * Base UAssetEditor class for character simulation asset editors (e.g. cloth, hair, flesh).
 * 
 * Our current asset editor guidelines ask us to place as little business logic as possible
 * into the class, instead putting as much of the non-UI code into the subsystem as possible,
 * and the UI code into the toolkit (which this class owns).
 *
 * However, since we're using a mode and the Interactive Tools Framework, a lot of our business logic
 * ends up inside the mode and the tools, not the subsystem. The front-facing code is mostly in
 * the asset editor toolkit, though the mode toolkit has most of the things that deal with the toolbar
 * on the left. 
 * 
 */

UCLASS(MinimalAPI, Abstract)
class UBaseCharacterFXEditor : public UAssetEditor
{
	GENERATED_BODY()

public:

	/** Retrieve stored objects to edit */
	UE_API virtual void GetObjectsToEdit(TArray<UObject*>& OutObjects) override;
	
	// Inherited from UAssetEditor and called in UAssetEditor::Initialize. Override this to create a subclass of FBaseCharacterFXEditorToolkit
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() PURE_VIRTUAL(UBaseCharacterFXEditor::CreateToolkit, return TSharedPtr<FBaseAssetToolkit>(););

	/** Store objects to edit and call UAssetEditor::Initialize */
	UE_API virtual void Initialize(const TArray<TObjectPtr<UObject>>& InObjects, const TSubclassOf<AActor>& InPreviewClass = {});

	/** Returns the asset editor instance interface, so that its window can be focused, for example. */
	UE_API virtual IAssetEditorInstance* GetInstanceInterface();

	// Note that things like CloseWindow, FocusWindow, and IsPrimaryEditor seem to be called
	// on the toolkit, not the editor class, so they are overriden there.

protected:
	
	UPROPERTY()
	TArray<TObjectPtr<UObject>> OriginalObjectsToEdit;
};

#undef UE_API
