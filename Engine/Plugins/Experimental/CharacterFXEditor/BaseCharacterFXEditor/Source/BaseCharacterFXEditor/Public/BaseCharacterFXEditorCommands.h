// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/InteractiveToolsCommands.h"

/**
* Home for FUICommandInfos used by the editor
*/

template<typename CommandContextType>
class TBaseCharacterFXEditorCommands : public TInteractiveToolCommands<CommandContextType>
{

public:

	// TInteractiveToolCommands<> interface
	// If you override this function, call this base version to set up Accept/Cancel buttons
	virtual void RegisterCommands() override
	{
		TInteractiveToolCommands<CommandContextType>::RegisterCommands();

#define LOCTEXT_NAMESPACE "TBaseCharacterFXEditorCommands"
		// These allow us to link up to pressed keys
		UI_COMMAND(AcceptOrCompleteActiveTool, "Accept", "Accept the active tool", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter));
		UI_COMMAND(CancelOrCompleteActiveTool, "Cancel", "Cancel the active tool", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
#undef LOCTEXT_NAMESPACE
	}

	TSharedPtr<FUICommandInfo> AcceptOrCompleteActiveTool;
	TSharedPtr<FUICommandInfo> CancelOrCompleteActiveTool;

protected:

	// Call this from subclass default constructor
	TBaseCharacterFXEditorCommands(const FName InContextName, const FText& InContextDesc, const FName InContextParent, const FName InStyleSetName) :
		TInteractiveToolCommands<CommandContextType>(InContextName, InContextDesc, InContextParent, InStyleSetName)
	{}

};
