// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tools/UEdMode.h"
#include "BaseCharacterFXEditorMode.generated.h"

#define UE_API BASECHARACTERFXEDITOR_API

class FToolTargetTypeRequirements;
class UToolTarget;
class UDynamicMeshComponent;
class UBaseCharacterFXEditorSubsystem;

/**
* The CharacterFX EditorMode stores much of the inter-tool state, including ToolTargets.
*/

UCLASS(MinimalAPI, Abstract)
class UBaseCharacterFXEditorMode : public UEdMode
{
	GENERATED_BODY()

public:

	// UEdMode overrides
	UE_API virtual void Enter() override;
	UE_API virtual void Exit() override;

	// Called by the corresponding EditorToolkit. Creates ToolTargets from OriginalObjectsToEdit.
	// Override to add custom per-target setup.
	UE_API virtual void InitializeTargets(const TArray<TObjectPtr<UObject>>& AssetsIn);

	virtual FBox SceneBoundingBox() const
	{
		return FBox();
	}

protected:

	// Override these to specify supported ToolTargets and tools
	virtual void AddToolTargetFactories() PURE_VIRTUAL(UBaseCharacterFXEditorMode::AddToolTargetFactories, );
	virtual void RegisterTools()  PURE_VIRTUAL(UBaseCharacterFXEditorMode::RegisterTools, );
//	virtual UBaseCharacterFXEditorSubsystem* GetEditorSubsystem()  PURE_VIRTUAL(UBaseCharacterFXEditorMode::GetEditorSubsystem, return nullptr;);

	// Called by InitializeTargets; override to populate tool targets for given assets
	virtual void CreateToolTargets(const TArray<TObjectPtr<UObject>>& AssetsIn) PURE_VIRTUAL(UBaseCharacterFXEditorMode::CreateToolTargets, );

	// Handles nested tools and notifies InteractiveToolsContext
	UE_API void AcceptActiveToolActionOrTool();
	UE_API void CancelActiveToolActionOrTool();

	/**
	 * Stores original input objects
	 */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> OriginalObjectsToEdit;

	/**
	 * Tool targets created from OriginalObjectsToEdit (and 1:1 with that array) that provide us with dynamic meshes
	 */
	UPROPERTY()
	TArray<TObjectPtr<UToolTarget>> ToolTargets;
};	

#undef UE_API
