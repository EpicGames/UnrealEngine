// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Toolkits/BaseToolkit.h"
#include "StatusBarSubsystem.h"

#define UE_API BASECHARACTERFXEDITOR_API

class SBorder;
class STextBlock;


/**
 * The editor mode toolkit is responsible for the panel on the side in the editor
 * that shows mode and tool properties. Tool buttons would go in Init().
 */

class FBaseCharacterFXEditorModeToolkit : public FModeToolkit
{
public:

	UE_API FBaseCharacterFXEditorModeToolkit();
	UE_API ~FBaseCharacterFXEditorModeToolkit();

	// FModeToolkit
	UE_API virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
	
	/** Active tool name to display in the ViewportOverlay */
	virtual FText GetActiveToolDisplayName() const override { return ActiveToolName; }

	/** Returns the Mode specific tabs in the mode toolbar **/
	UE_API virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const override;

    /** Returns human readable name for the specific palette tool category **/
	UE_API virtual FText GetToolPaletteDisplayName(FName PaletteName) const override;
 
	/** Informs the built-in FModeToolkit palette building routines about how we want the toolbars to be set up **/
	virtual bool HasIntegratedToolPalettes() const override { return true; }
	virtual bool HasExclusiveToolPalettes() const override { return false; }

	/** Returns the mode's toolbox */
	virtual TSharedPtr<class SWidget> GetInlineContent() const override { return ToolkitWidget; }

	/** Set/clear notification message area */
	UE_API virtual void PostNotification(const FText& Message);
	UE_API virtual void ClearNotification();

	/** Set/clear warning message area */
	UE_API virtual void PostWarning(const FText& Message);
	UE_API virtual void ClearWarning();

protected:

	// Should be implemented in the concrete subclass since it will likely have to know where to look for UI resources
	virtual const FSlateBrush* GetActiveToolIcon(const FString& Identifier) const PURE_VIRTUAL(FBaseCharacterFXEditorModeToolkit::GetActiveToolIcon, return nullptr;);

	// The editor can have multiple tabs in the mode toolbar. ToolsTabName is the "Tools" tab maintained by this
	// base class. PaletteNames_Standard is the set of all tab names in the concrete editor. Set these in the concrete
	// subclass when adding additional toolbar tabs
	UE_API const static FName ToolsTabName;
	UE_API const static TArray<FName> PaletteNames_Standard;

	// FModeToolkit
	UE_API virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;

	// A place for tools to write out any warnings
	TSharedPtr<STextBlock> ToolWarningArea;

	// A container for the tool settings that is populated by the DetailsView managed in FModeToolkit
	TSharedPtr<SBorder> ToolDetailsContainer;

	// Contains the widget container for the Accept/Cancel buttons, etc. for tools
	TSharedPtr<SWidget> ViewportOverlayWidget;

	// Tool name, icon and message to display in viewport
	FText ActiveToolName;
	const FSlateBrush* ActiveToolIcon;
	FStatusBarMessageHandle ActiveToolMessageHandle;

	UE_API void UpdateActiveToolProperties();
	UE_API void InvalidateCachedDetailPanelState(UObject* ChangedObject);
};

#undef UE_API
