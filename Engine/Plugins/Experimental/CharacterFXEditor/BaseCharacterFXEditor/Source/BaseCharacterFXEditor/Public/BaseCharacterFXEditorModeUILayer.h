// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorModeUILayer.h"
#include "BaseCharacterFXEditorModeUILayer.generated.h"

#define UE_API BASECHARACTERFXEDITOR_API

/** Handles the hosting of additional toolkits, such as the mode toolkit, within the CharacterFXEditor's toolkit. **/

class FBaseCharacterFXEditorModeUILayer : public FAssetEditorModeUILayer
{
public:
	UE_API FBaseCharacterFXEditorModeUILayer(const IToolkitHost* InToolkitHost);
	UE_API void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	UE_API void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;

	UE_API void SetModeMenuCategory(TSharedPtr<FWorkspaceItem> MenuCategoryIn);
	UE_API TSharedPtr<FWorkspaceItem> GetModeMenuCategory() const override;

protected:

	TSharedPtr<FWorkspaceItem> CharacterFXEditorMenuCategory;

};

/** 
 * Interchange layer to manage built in tab locations within the editor's layout. 
 */

UCLASS(MinimalAPI)
class UBaseCharacterFXEditorUISubsystem : public UAssetEditorUISubsystem
{
	GENERATED_BODY()
public:

	// Adds RegisterLayoutExtensions as a callback to the module corresponding to GetModuleName()
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	// Removes RegisterLayoutExtensions callback from the module corresponding to GetModuleName()
	UE_API virtual void Deinitialize() override;

	// Docks the editor Mode tab in the Editor side panel area of the layout
	UE_API virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) override;

	// Identifier for the Layout extension in BaseCharacterFXEditorToolkit's StandaloneDefaultLayout
	static UE_API const FName EditorSidePanelAreaName;

protected:

	// OVERRIDE THIS
	virtual FName GetModuleName() const
	{
		return FName("");
	}

};

#undef UE_API
