// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API BASECHARACTERFXEDITOR_API

class FLayoutExtender;

class FBaseCharacterFXEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	// Used by the EditorUISubsystem
	DECLARE_EVENT_OneParam(FExampleCharacterFXEditorModule, FOnRegisterLayoutExtensions, FLayoutExtender&);
	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() { return RegisterLayoutExtensions; }

private:

	FOnRegisterLayoutExtensions	RegisterLayoutExtensions;

};

#undef UE_API
