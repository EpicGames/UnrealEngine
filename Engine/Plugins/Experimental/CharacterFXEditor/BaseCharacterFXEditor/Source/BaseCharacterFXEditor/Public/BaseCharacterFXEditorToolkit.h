// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/BaseAssetToolkit.h"
#include "BaseCharacterFXEditorModeUILayer.h"
#include "PreviewScene.h"

#define UE_API BASECHARACTERFXEDITOR_API

class UBaseCharacterFXEditorMode;

/**
 * The toolkit is supposed to act as the UI manager for the asset editor. It's responsible 
 * for setting up viewports and most toolbars, except for the internals of the mode panel.
 * However, because the toolkit also sets up the mode manager, and much of the important
 * state is held in the mode managed by the mode manager, the toolkit also ends up
 * initializing the mode.
 * Thus, the FBaseCharacterFXEditorToolkit ends up being the central place for the CharacterFX Asset Editor setup.
 */

class FBaseCharacterFXEditorToolkit : public FBaseAssetToolkit
{
public:
	    
	UE_API FBaseCharacterFXEditorToolkit(UAssetEditor* InOwningAssetEditor, const FName& ModuleName);

	UE_API virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	UE_API virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	UE_API virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	UE_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;

	UE_API virtual AssetEditorViewportFactoryFunction GetViewportDelegate() override;
	UE_API virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder = INDEX_NONE) override;
	UE_API virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget) override;

protected:

	// Override this to get the Mode ID of the concrete CharacterFXEditorMode
	virtual FEditorModeID GetEditorModeId() const PURE_VIRTUAL(FBaseCharacterFXEditorToolkit::GetEditorModeId, return FEditorModeID("EM_ERROR"););

	// Helpers for PostInitAssetEditor
	
	// Override this to create the concrete CharacterFXEditorModeUILayer
	UE_API virtual void CreateEditorModeUILayer();

	// Override this to provide the EdMode with any custom data it needs
	UE_API virtual void InitializeEdMode(UBaseCharacterFXEditorMode* EdMode);

	UE_API virtual void PostInitAssetEditor() override;

	// Called from FBaseAssetToolkit::CreateWidgets.
	UE_API virtual void CreateEditorModeManager() override;
	
	// Called from FBaseAssetToolkit::CreateWidgets to populate ViewportClient, but otherwise only used 
	// in our own viewport delegate
	UE_API virtual TSharedPtr<FEditorViewportClient> CreateEditorViewportClient() const override;

	// Handles the hosting of additional toolkits, such as the mode toolkit
	TSharedPtr<FBaseCharacterFXEditorModeUILayer> ModeUILayer;

	// PreviewScene showing the objects being edited
	TSharedPtr<FPreviewScene> ObjectScene;

	// A default "CharacterFXEditor" category is created in RegisterTabSpawners. Override that function to set this to a concrete class type
	TSharedPtr<FWorkspaceItem> EditorMenuCategory;

	// Boolean to force the construction of the viewport tab even if the layout is not adding it 
	bool bForceViewportTab = true;
};

#undef UE_API
