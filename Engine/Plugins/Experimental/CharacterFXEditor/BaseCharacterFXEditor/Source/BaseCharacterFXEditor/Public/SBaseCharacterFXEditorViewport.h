// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SAssetEditorViewport.h"

#define UE_API BASECHARACTERFXEDITOR_API

class SBaseCharacterFXEditorViewport : public SAssetEditorViewport
{
public:

	// These allow the toolkit to add an accept/cancel overlay when needed. PopulateViewportOverlays
	// is not helpful here because that gets called just once.
	UE_API virtual void AddOverlayWidget(TSharedRef<SWidget> OverlaidWidget, int32 ZOrder = INDEX_NONE);
	UE_API virtual void RemoveOverlayWidget(TSharedRef<SWidget> OverlaidWidget);
};

#undef UE_API
